// Copyright Epic Games, Inc. All Rights Reserved.

#include "IMeshBuilderCommonModule.h"
#include "Modules/ModuleManager.h"

class FMeshBuilderCommonModule : public IMeshBuilderCommonModule
{
public:

	FMeshBuilderCommonModule()
	{
	}

	virtual void StartupModule() override
	{
		// Register any modular features here
	}

	virtual void ShutdownModule() override
	{
		// Unregister any modular features here
	}

};

IMPLEMENT_MODULE(FMeshBuilderCommonModule, MeshBuilderCommon );
