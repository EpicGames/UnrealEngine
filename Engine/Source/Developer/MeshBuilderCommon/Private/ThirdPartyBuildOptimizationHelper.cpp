// Copyright Epic Games, Inc. All Rights Reserved.
#include "ThirdPartyBuildOptimizationHelper.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"

#include "NvTriStrip.h"
#include "forsythtriangleorderoptimizer.h"

#ifndef _SILENCE_STDEXT_HASH_DEPRECATION_WARNINGS
#define _SILENCE_STDEXT_HASH_DEPRECATION_WARNINGS 1
#endif

#include "nvtess.h"

namespace BuildOptimizationThirdParty
{
	// CVars
	static TAutoConsoleVariable<int32> CVarTriangleOrderOptimization(
		TEXT("r.TriangleOrderOptimization"),
		1,
		TEXT("Controls the algorithm to use when optimizing the triangle order for the post-transform cache.\n")
		TEXT("0: Use NVTriStrip (slower)\n")
		TEXT("1: Use Forsyth algorithm (fastest)(default)")
		TEXT("2: No triangle order optimization. (least efficient, debugging purposes only)"),
		ECVF_Default);

	namespace NvTriStripHelper
	{
		/**
		* Converts 16 bit indices to 32 bit prior to passing them into the real GenerateStrips util method
		*/
		void GenerateStrips(
			const uint8* Indices,
			bool Is32Bit,
			const uint32 NumIndices,
			PrimitiveGroup** PrimGroups,
			uint32* NumGroups
		)
		{
			if (Is32Bit)
			{
				GenerateStrips((uint32*)Indices, NumIndices, PrimGroups, NumGroups);
			}
			else
			{
				// convert to 32 bit
				uint32 Idx;
				TArray<uint32> NewIndices;
				NewIndices.AddUninitialized(NumIndices);
				for (Idx = 0; Idx < NumIndices; ++Idx)
				{
					NewIndices[Idx] = ((uint16*)Indices)[Idx];
				}
				GenerateStrips(NewIndices.GetData(), NumIndices, PrimGroups, NumGroups);
			}

		}

		/**
		* Orders a triangle list for better vertex cache coherency.
		*
		* *** WARNING: This is safe to call for multiple threads IF AND ONLY IF all
		* threads call SetListsOnly(true) and SetCacheSize(CACHESIZE_GEFORCE3). If
		* NvTriStrip is ever used with different settings the library will need
		* some modifications to be thread-safe. ***
		*/
		template<typename IndexDataType, typename Allocator>
		void CacheOptimizeIndexBuffer(TArray<IndexDataType, Allocator>& Indices)
		{
			static_assert(sizeof(IndexDataType) == 2 || sizeof(IndexDataType) == 4, "Indices must be short or int.");

			PrimitiveGroup*	PrimitiveGroups = NULL;
			uint32			NumPrimitiveGroups = 0;
			bool Is32Bit = sizeof(IndexDataType) == 4;

			SetListsOnly(true);
			SetCacheSize(CACHESIZE_GEFORCE3);

			GenerateStrips((uint8*)Indices.GetData(), Is32Bit, Indices.Num(), &PrimitiveGroups, &NumPrimitiveGroups);

			Indices.Empty();
			Indices.AddUninitialized(PrimitiveGroups->numIndices);

			if (Is32Bit)
			{
				FMemory::Memcpy(Indices.GetData(), PrimitiveGroups->indices, Indices.Num() * sizeof(IndexDataType));
			}
			else
			{
				for (uint32 I = 0; I < PrimitiveGroups->numIndices; ++I)
				{
					Indices[I] = (uint16)PrimitiveGroups->indices[I];
				}
			}

			delete[] PrimitiveGroups;
		}

		/**
		* Provides skeletal mesh render data to the NVIDIA tessellation library.
		*/
		class FSkeletalMeshNvRenderBuffer : public nv::RenderBuffer
		{
		public:

			/** Construct from static mesh render buffers. */
			FSkeletalMeshNvRenderBuffer(
				const TArray<FSoftSkinVertex>& InVertexBuffer,
				const uint32 InTexCoordCount,
				const TArray<uint32>& Indices)
				: VertexBuffer(InVertexBuffer)
				, TexCoordCount(InTexCoordCount)
			{
				mIb = new nv::IndexBuffer((void*)Indices.GetData(), nv::IBT_U32, Indices.Num(), false);
			}

			/** Retrieve the position and first texture coordinate of the specified index. */
			virtual nv::Vertex getVertex(unsigned int Index) const
			{
				nv::Vertex Vertex;

				check(Index < (unsigned int)VertexBuffer.Num());

				const FSoftSkinVertex& SrcVertex = VertexBuffer[Index];

				Vertex.pos.x = SrcVertex.Position.X;
				Vertex.pos.y = SrcVertex.Position.Y;
				Vertex.pos.z = SrcVertex.Position.Z;

				if (TexCoordCount > 0)
				{
					Vertex.uv.x = SrcVertex.UVs[0].X;
					Vertex.uv.y = SrcVertex.UVs[0].Y;
				}
				else
				{
					Vertex.uv.x = 0.0f;
					Vertex.uv.y = 0.0f;
				}

				return Vertex;
			}

		private:
			/** The vertex buffer for the skeletal mesh. */
			const TArray<FSoftSkinVertex>& VertexBuffer;
			const uint32 TexCoordCount;

			/** Copying is forbidden. */
			FSkeletalMeshNvRenderBuffer(const FSkeletalMeshNvRenderBuffer&);
			FSkeletalMeshNvRenderBuffer& operator=(const FSkeletalMeshNvRenderBuffer&);
		};

		/**
		* Provides static mesh render data to the NVIDIA tessellation library.
		*/
		class FStaticMeshNvRenderBuffer : public nv::RenderBuffer
		{
		public:

			/** Construct from static mesh render buffers. */
			FStaticMeshNvRenderBuffer(
				const FPositionVertexBuffer& InPositionVertexBuffer,
				const FStaticMeshVertexBuffer& InVertexBuffer,
				const TArray<uint32>& Indices);

			/** Retrieve the position and first texture coordinate of the specified index. */
			virtual nv::Vertex getVertex(unsigned int Index) const;

		private:

			/** The position vertex buffer for the static mesh. */
			const FPositionVertexBuffer& PositionVertexBuffer;

			/** The vertex buffer for the static mesh. */
			const FStaticMeshVertexBuffer& VertexBuffer;

			/** Copying is forbidden. */
			FStaticMeshNvRenderBuffer(const FStaticMeshNvRenderBuffer&);
			FStaticMeshNvRenderBuffer& operator=(const FStaticMeshNvRenderBuffer&);
		};

		FStaticMeshNvRenderBuffer::FStaticMeshNvRenderBuffer(
			const FPositionVertexBuffer& InPositionVertexBuffer,
			const FStaticMeshVertexBuffer& InVertexBuffer,
			const TArray<uint32>& Indices)
			: PositionVertexBuffer(InPositionVertexBuffer)
			, VertexBuffer(InVertexBuffer)
		{
			check(PositionVertexBuffer.GetNumVertices() == VertexBuffer.GetNumVertices());
			mIb = new nv::IndexBuffer((void*)Indices.GetData(), nv::IBT_U32, Indices.Num(), false);
		}

		nv::Vertex FStaticMeshNvRenderBuffer::getVertex(unsigned int Index) const
		{
			nv::Vertex Vertex;

			check(Index < PositionVertexBuffer.GetNumVertices());

			const FVector3f& Position = PositionVertexBuffer.VertexPosition(Index);
			Vertex.pos.x = Position.X;
			Vertex.pos.y = Position.Y;
			Vertex.pos.z = Position.Z;

			if (VertexBuffer.GetNumTexCoords())
			{
				const FVector2f UV = VertexBuffer.GetVertexUV(Index, 0);
				Vertex.uv.x = UV.X;
				Vertex.uv.y = UV.Y;
			}
			else
			{
				Vertex.uv.x = 0.0f;
				Vertex.uv.y = 0.0f;
			}

			return Vertex;
		}

		void BuildStaticAdjacencyIndexBuffer(
			const FPositionVertexBuffer& PositionVertexBuffer,
			const FStaticMeshVertexBuffer& VertexBuffer,
			const TArray<uint32>& Indices,
			TArray<uint32>& OutPnAenIndices
		)
		{
			if (Indices.Num())
			{
				FStaticMeshNvRenderBuffer StaticMeshRenderBuffer(PositionVertexBuffer, VertexBuffer, Indices);
				nv::IndexBuffer* PnAENIndexBuffer = nv::tess::buildTessellationBuffer(&StaticMeshRenderBuffer, nv::DBM_PnAenDominantCorner, true);
				check(PnAENIndexBuffer);
				const int32 IndexCount = (int32)PnAENIndexBuffer->getLength();
				OutPnAenIndices.Empty(IndexCount);
				OutPnAenIndices.AddUninitialized(IndexCount);
				for (int32 Index = 0; Index < IndexCount; ++Index)
				{
					OutPnAenIndices[Index] = (*PnAENIndexBuffer)[Index];
				}
				delete PnAENIndexBuffer;
			}
			else
			{
				OutPnAenIndices.Empty();
			}
		}

		void BuildSkeletalAdjacencyIndexBuffer(
			const TArray<FSoftSkinVertex>& VertexBuffer,
			const uint32 TexCoordCount,
			const TArray<uint32>& Indices,
			TArray<uint32>& OutPnAenIndices
		)
		{
			if (Indices.Num())
			{
				FSkeletalMeshNvRenderBuffer SkeletalMeshRenderBuffer(VertexBuffer, TexCoordCount, Indices);
				nv::IndexBuffer* PnAENIndexBuffer = nv::tess::buildTessellationBuffer(&SkeletalMeshRenderBuffer, nv::DBM_PnAenDominantCorner, true);
				check(PnAENIndexBuffer);
				const int32 IndexCount = (int32)PnAENIndexBuffer->getLength();
				OutPnAenIndices.Empty(IndexCount);
				OutPnAenIndices.AddUninitialized(IndexCount);
				for (int32 Index = 0; Index < IndexCount; ++Index)
				{
					OutPnAenIndices[Index] = (*PnAENIndexBuffer)[Index];
				}
				delete PnAENIndexBuffer;
			}
			else
			{
				OutPnAenIndices.Empty();
			}
		}
	}

	namespace ForsythHelper
	{
		void OptimizeFaces(
			const uint8* Indices,
			bool Is32Bit,
			const uint32 NumIndices,
			uint32 NumVertices,
			uint32* OutIndices,
			uint16 CacheSize
		)
		{
			if (Is32Bit)
			{
				Forsyth::OptimizeFaces((uint32*)Indices, NumIndices, NumVertices, OutIndices, CacheSize);
			}
			else
			{
				// convert to 32 bit
				uint32 Idx;
				TArray<uint32> NewIndices;
				NewIndices.AddUninitialized(NumIndices);
				for (Idx = 0; Idx < NumIndices; ++Idx)
				{
					NewIndices[Idx] = ((uint16*)Indices)[Idx];
				}
				Forsyth::OptimizeFaces(NewIndices.GetData(), NumIndices, NumVertices, OutIndices, CacheSize);
			}

		}

		template<typename IndexDataType, typename Allocator>
		void CacheOptimizeIndexBuffer(TArray<IndexDataType, Allocator>& Indices)
		{
			static_assert(sizeof(IndexDataType) == 2 || sizeof(IndexDataType) == 4, "Indices must be short or int.");
			bool Is32Bit = sizeof(IndexDataType) == 4;

			// Count the number of vertices
			uint32 NumVertices = 0;
			for (int32 Index = 0; Index < Indices.Num(); ++Index)
			{
				if (Indices[Index] > NumVertices)
				{
					NumVertices = Indices[Index];
				}
			}
			NumVertices += 1;

			TArray<uint32> OptimizedIndices;
			OptimizedIndices.AddUninitialized(Indices.Num());
			uint16 CacheSize = 32;
			OptimizeFaces((uint8*)Indices.GetData(), Is32Bit, Indices.Num(), NumVertices, OptimizedIndices.GetData(), CacheSize);

			if (Is32Bit)
			{
				FMemory::Memcpy(Indices.GetData(), OptimizedIndices.GetData(), Indices.Num() * sizeof(IndexDataType));
			}
			else
			{
				for (int32 I = 0; I < OptimizedIndices.Num(); ++I)
				{
					Indices[I] = (uint16)OptimizedIndices[I];
				}
			}
		}
	}

	void CacheOptimizeIndexBuffer(TArray<uint16>& Indices)
	{
		bool bDisableTriangleOrderOptimization = (CVarTriangleOrderOptimization.GetValueOnAnyThread(true) == 2);
		bool bUsingNvTriStrip = !bDisableTriangleOrderOptimization && (CVarTriangleOrderOptimization.GetValueOnAnyThread(true) == 0);

		if (bUsingNvTriStrip)
		{
			NvTriStripHelper::CacheOptimizeIndexBuffer(Indices);
		}
		else if (!bDisableTriangleOrderOptimization)
		{
			ForsythHelper::CacheOptimizeIndexBuffer(Indices);
		}
	}

	void CacheOptimizeIndexBuffer(TArray<uint32>& Indices)
	{
		bool bDisableTriangleOrderOptimization = (CVarTriangleOrderOptimization.GetValueOnAnyThread(true) == 2);
		bool bUsingNvTriStrip = !bDisableTriangleOrderOptimization && (CVarTriangleOrderOptimization.GetValueOnAnyThread(true) == 0);

		if (bUsingNvTriStrip)
		{
			NvTriStripHelper::CacheOptimizeIndexBuffer(Indices);
		}
		else if (!bDisableTriangleOrderOptimization)
		{
			ForsythHelper::CacheOptimizeIndexBuffer(Indices);
		}
	}
}
