// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "HAL/IConsoleManager.h"
#include "Rendering/PositionVertexBuffer.h"
#include "Rendering/StaticMeshVertexBuffer.h"
#include "Rendering/SkeletalMeshLODModel.h"

namespace BuildOptimizationThirdParty
{
	MESHBUILDERCOMMON_API void CacheOptimizeIndexBuffer(TArray<uint16>& Indices);
	MESHBUILDERCOMMON_API void CacheOptimizeIndexBuffer(TArray<uint32>& Indices);

	/*------------------------------------------------------------------------------
	NVTriStrip for cache optimizing index buffers.
	------------------------------------------------------------------------------*/

	namespace NvTriStripHelper
	{
		/*****************************
		 * Skeletal mesh helpers
		 */
		MESHBUILDERCOMMON_API void BuildStaticAdjacencyIndexBuffer(
			const FPositionVertexBuffer& PositionVertexBuffer,
			const FStaticMeshVertexBuffer& VertexBuffer,
			const TArray<uint32>& Indices,
			TArray<uint32>& OutPnAenIndices
		);

		MESHBUILDERCOMMON_API void BuildSkeletalAdjacencyIndexBuffer(
			const TArray<FSoftSkinVertex>& VertexBuffer,
			const uint32 TexCoordCount,
			const TArray<uint32>& Indices,
			TArray<uint32>& OutPnAenIndices
		);
	}
}
